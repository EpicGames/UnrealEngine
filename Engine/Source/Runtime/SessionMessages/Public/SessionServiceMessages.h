// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "SessionServiceMessages.generated.h"


/* Session discovery messages
 *****************************************************************************/

/**
 * Implements a message that is published to discover existing application sessions.
 */
USTRUCT()
struct FSessionServicePing
{
	GENERATED_USTRUCT_BODY()

	/** The name of the user who sent this ping. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString UserName;

	/** Default constructor. */
	FSessionServicePing() { }

	/** Creates and initializes a new instance. */
	FSessionServicePing(const FString& InUserName)
		: UserName(InUserName)
	{ }
};


/**
 * Implements a message that is published in response to FSessionServicePing.
 */
USTRUCT()
struct FSessionServicePong
{
	GENERATED_USTRUCT_BODY()

	/** Indicates whether the pinging user is authorized to interact with this session. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool Authorized = true;

	/** Holds the application's build date. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString BuildDate;

	/** Holds the name of the device that the application is running on. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString DeviceName;

	/** Holds the application's instance identifier. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid InstanceId;

	/** Holds the application's instance name. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString InstanceName;

	/** Holds the name of the platform that the application is running on. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString PlatformName;

	/** Holds the identifier of the session that the application belongs to. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid SessionId;

	/** Holds the user defined name of the session. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString SessionName;

	/** Holds the name of the user that started the session. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString SessionOwner;

	/** Indicates whether the application is the only one in that session. */
	UPROPERTY(EditAnywhere, Category="Message")
	bool Standalone = true;
};


/* Session status messages
 *****************************************************************************/

/**
 * Implements a message that contains a console log entry.
 */
USTRUCT()
struct FSessionServiceLog
{
	GENERATED_USTRUCT_BODY()

	/** Holds the log message category. */
	UPROPERTY(EditAnywhere, Category="Message")
	FName Category;

	/** Holds the log message data. */
	UPROPERTY(EditAnywhere, Category="Message")
	FString Data;

	/** Holds the application instance identifier. */
	UPROPERTY(EditAnywhere, Category="Message")
	FGuid InstanceId;

	/** Holds the time in seconds since the application was started. */
	UPROPERTY(EditAnywhere, Category="Message")
	double TimeSeconds = 0.0;

	/** Holds the log message's verbosity level. */
	UPROPERTY(EditAnywhere, Category="Message")
	uint8 Verbosity;

public:

	/** Default constructor. */
	FSessionServiceLog() : TimeSeconds(0.0), Verbosity(0) { }

	/**
	 * Creates and initializes a new instance.
	 */
	FSessionServiceLog(const FName& InCategory, const FString& InData, const FGuid& InInstanceId, double InTimeSeconds, uint8 InVerbosity)
		: Category(InCategory)
		, Data(InData)
		, InstanceId(InInstanceId)
		, TimeSeconds(InTimeSeconds)
		, Verbosity(InVerbosity)
	{ }
};


/**
 * Implements a message to subscribe to an application's console log.
 */
USTRUCT()
struct FSessionServiceLogSubscribe
{
	GENERATED_USTRUCT_BODY()
};


/**
 * Implements a message to unsubscribe from an application's console log.
 */
USTRUCT()
struct FSessionServiceLogUnsubscribe
{
	GENERATED_USTRUCT_BODY()
};
